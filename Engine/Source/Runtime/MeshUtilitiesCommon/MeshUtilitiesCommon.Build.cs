// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    /**
     * Common algorithms and data structures used by MeshUtilities, MeshDescriptionOperations and other mesh-related modules.
     * This module must have minimal dependencies to allow it being used by various modules without introducing circular references.
     */
    public class MeshUtilitiesCommon : ModuleRules
    {
        public MeshUtilitiesCommon(ReadOnlyTargetRules Target) : base(Target)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                }
            );
        }
    }
}
