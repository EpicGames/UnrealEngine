// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayoutUV.h"
#include "Algo/IntroSort.h"
#include "Async/Async.h"
#include "DisjointSet.h"
#include "OverlappingCorners.h"
#include "HAL/PlatformTime.h"
#include "Misc/App.h"
#include "Misc/SecureHash.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, MeshUtilitiesCommon)

DEFINE_LOG_CATEGORY_STATIC(LogLayoutUV, Log, All);

#define CHART_JOINING	1

#define NEW_UVS_ARE_SAME				UE_THRESH_POINTS_ARE_SAME
#define LEGACY_UVS_ARE_SAME				UE_THRESH_UVS_ARE_SAME
#define UVLAYOUT_THRESH_UVS_ARE_SAME	(GetUVEqualityThreshold())

TAtomic<uint64> FLayoutUV::FindBestPackingCount(0);
TAtomic<uint64> FLayoutUV::FindBestPackingCycles(0);
TAtomic<uint64> FLayoutUV::FindBestPackingEfficiency(0);

FLayoutUV::FLayoutUV( IMeshView& InMeshView )
	: MeshView( InMeshView )
	, LayoutVersion( ELightmapUVVersion::Latest )
	, PackedTextureResolution(0)
{}


/** FIRST PASS: Given a Mesh, build the associated set of charts */
struct FLayoutUV::FChartFinder
{
	FChartFinder(IMeshView& InMeshView, ELightmapUVVersion InLayoutVersion);

	int32 FindCharts( const FOverlappingCorners& OverlappingCorners, TArray< FVector2f >& TexCoords, TArray< uint32 >& SortedTris, TArray< FMeshChart >& Charts );

private:
	bool PositionsMatch( uint32 a, uint32 b ) const;
	bool NormalsMatch( uint32 a, uint32 b ) const;
	bool UVsMatch( uint32 a, uint32 b ) const;
	bool VertsMatch( uint32 a, uint32 b ) const;
	float TriangleUVArea( uint32 Tri ) const;
	void DisconnectChart( TArray< FMeshChart >& Charts, FMeshChart& Chart, uint32 Side );
	float GetUVEqualityThreshold() const;

private:
	IMeshView& MeshView;
	ELightmapUVVersion LayoutVersion;

	int32 NextMeshChartId;
};


/** SECOND PASS: Given a set of charts, pack them in the UV space */
struct FLayoutUV::FChartPacker
{
	FChartPacker(IMeshView& InMeshView, ELightmapUVVersion InLayoutVersion, uint32 TextureResolution);

	bool FindBestPacking(const TArray< FVector2f >& TexCoords, const TArray< uint32 >& SortedTris, TArray< FMeshChart >& AllCharts);

private:
	void ScaleCharts( TArray< FMeshChart >& Charts, float UVScale );
	bool PackCharts( TArray< FMeshChart >& Charts, float UVScale, const TArray< FVector2f >& TexCoords, const TArray< uint32 >& SortedTris, float& OutEfficiency, TAtomic<bool>& bAbort, bool bTrace);
	void OrientChart( FMeshChart& Chart, int32 Orientation );
	void RasterizeChart( const FMeshChart& Chart, const TArray< FVector2f >& TexCoords, const TArray< uint32 >& SortedTris, uint32 RectW, uint32 RectH, FAllocator2D& OutChartRaster );

private:
	IMeshView& MeshView;
	ELightmapUVVersion LayoutVersion;

	uint32 TextureResolution;
	float TotalUVArea;
};

FLayoutUV::FChartPacker::FChartPacker(IMeshView& InMeshView, ELightmapUVVersion InLayoutVersion, uint32 TextureResolution)
	: MeshView(InMeshView)
	, LayoutVersion(InLayoutVersion)
	, TextureResolution(TextureResolution)
	, TotalUVArea(0.0f)
{
}

int32 FLayoutUV::FindCharts(const FOverlappingCorners& OverlappingCorners)
{
	FChartFinder Finder(MeshView, LayoutVersion);
	return Finder.FindCharts(OverlappingCorners, MeshTexCoords, MeshSortedTris, MeshCharts);
}

FLayoutUV::FChartFinder::FChartFinder(IMeshView& InMeshView, ELightmapUVVersion InLayoutVersion)
	: MeshView(InMeshView)
	, LayoutVersion(InLayoutVersion)
	, NextMeshChartId( 0 )
{}

int32 FLayoutUV::FChartFinder::FindCharts( const FOverlappingCorners& OverlappingCorners, TArray< FVector2f >& TexCoords, TArray< uint32 >& SortedTris, TArray< FMeshChart >& Charts )
{
	double Begin = FPlatformTime::Seconds();

	uint32 NumIndexes = MeshView.GetNumIndices();
	uint32 NumTris = NumIndexes / 3;

	TArray< int32 > TranslatedMatches;
	TranslatedMatches.SetNumUninitialized( NumIndexes );
	TexCoords.SetNumUninitialized( NumIndexes );
	for( uint32 i = 0; i < NumIndexes; i++ )
	{
		TranslatedMatches[i] = -1;
		TexCoords[i] = MeshView.GetInputTexcoord(i);
	}

	// Build disjoint set
	FDisjointSet DisjointSet( NumTris );
	for( uint32 i = 0; i < NumIndexes; i++ )
	{
		const TArray<int32>& Overlapping = OverlappingCorners.FindIfOverlapping(i);
		for (int32 It : Overlapping)
		{
			uint32 j = It;
			if( j > i )
			{
				const uint32 TriI = i/3;
				const uint32 TriJ = j/3;

				bool bUnion = false;

#if CHART_JOINING
				bool bPositionMatch = PositionsMatch( i, j );
				if( bPositionMatch )
				{
					uint32 i1 = 3 * TriI + (i + 1) % 3;
					uint32 i2 = 3 * TriI + (i + 2) % 3;
					uint32 j1 = 3 * TriJ + (j + 1) % 3;
					uint32 j2 = 3 * TriJ + (j + 2) % 3;

					bool bEdgeMatch21 = PositionsMatch( i2, j1 );
					bool bEdgeMatch12 = PositionsMatch( i1, j2 );
					if( bEdgeMatch21 || bEdgeMatch12 )
					{
						uint32 ie = bEdgeMatch21 ? i2 : i1;
						uint32 je = bEdgeMatch21 ? j1 : j2;
							
						bool bUVMatch = UVsMatch( i, j ) && UVsMatch( ie, je );
						bool bUVWindingMatch = TriangleUVArea( TriI ) * TriangleUVArea( TriJ ) >= 0.0f;
						if( bUVMatch && bUVWindingMatch )
						{
							bUnion = true;
						}
						else if( NormalsMatch( i, j ) && NormalsMatch( ie, je ) )
						{
							// Chart edge
							FVector2f EdgeUVi = TexCoords[ie] - TexCoords[i];
							FVector2f EdgeUVj = TexCoords[je] - TexCoords[j];
							
							// Would these edges match if the charts were translated
							bool bTranslatedUVMatch = ( EdgeUVi - EdgeUVj ).IsNearlyZero(UVLAYOUT_THRESH_UVS_ARE_SAME);
							if( bTranslatedUVMatch )
							{
								// Note: may be mirrored
								
								// TODO should these be restricted to axis aligned edges?
								uint32 EdgeI = bEdgeMatch21 ? i2 : i;
								uint32 EdgeJ = bEdgeMatch21 ? j : j2;

								// Only allow one match per edge
								if( TranslatedMatches[ EdgeI ] < 0 &&
									TranslatedMatches[ EdgeJ ] < 0 )
								{
									TranslatedMatches[ EdgeI ] = EdgeJ;
									TranslatedMatches[ EdgeJ ] = EdgeI;
								}
							}
						}
					}
				}
#else
				if( VertsMatch( i, j ) )
				{
					// Edge must match as well (same winding)
					if( VertsMatch( 3 * TriI + (i - 1) % 3, 3 * TriJ + (j + 1) % 3 ) ||
						VertsMatch( 3 * TriI + (i + 1) % 3, 3 * TriJ + (j - 1) % 3 ) )
					{
						// Check for UV winding match too
						if( TriangleUVArea( TriI ) * TriangleUVArea( TriJ ) >= 0.0f )
						{
							bUnion = true;
						}
					}
				}
#endif

				if( bUnion )
				{
					// TODO solve spiral case by checking sets for UV overlap
					DisjointSet.Union( TriI, TriJ );
				}
			}
		}
	}

	// Sort tris by chart
	SortedTris.SetNumUninitialized( NumTris );
	for( uint32 i = 0; i < NumTris; i++ )
	{
		// Flatten disjoint set path
		DisjointSet.Find(i);
		SortedTris[i] = i;
	}

	struct FCompareTris
	{
		FDisjointSet* DisjointSet;

		FCompareTris( FDisjointSet* InDisjointSet )
		: DisjointSet( InDisjointSet )
		{}

		FORCEINLINE bool operator()( uint32 A, uint32 B ) const
		{
			return (*DisjointSet)[A] < (*DisjointSet)[B];
		}
	};

	Algo::IntroSort( SortedTris, FCompareTris( &DisjointSet ) );

	TMap< uint32, int32 > DisjointSetToChartMap;

	// Build Charts
	for( uint32 Tri = 0; Tri < NumTris; )
	{
		int32 i = Charts.AddUninitialized();
		FMeshChart& Chart = Charts[i];
		Chart.Id = NextMeshChartId++;

		Chart.MinUV = FVector2f( FLT_MAX, FLT_MAX );
		Chart.MaxUV = FVector2f( -FLT_MAX, -FLT_MAX );
		Chart.UVArea = 0.0f;
		Chart.WorldScale = FVector2f::ZeroVector;
		Chart.UVLengthSum = 0.0f;
		Chart.WorldLengthSum = 0.0f;
		FMemory::Memset( Chart.Join, 0xff );

		Chart.FirstTri = Tri;

		uint32 ChartID = DisjointSet[ SortedTris[ Tri ] ];
		DisjointSetToChartMap.Add( ChartID, i );

		for( ; Tri < NumTris && DisjointSet[ SortedTris[ Tri ] ] == ChartID; Tri++ )
		{
			// Calculate chart bounds
			FVector3f	Positions[3];
			FVector2f	UVs[3];
			for( int k = 0; k < 3; k++ )
			{
				uint32 Index = 3 * SortedTris[ Tri ] + k;

				Positions[k] = MeshView.GetPosition( Index );
				UVs[k] = TexCoords[ Index ];

				Chart.MinUV.X = FMath::Min( Chart.MinUV.X, UVs[k].X );
				Chart.MinUV.Y = FMath::Min( Chart.MinUV.Y, UVs[k].Y );
				Chart.MaxUV.X = FMath::Max( Chart.MaxUV.X, UVs[k].X );
				Chart.MaxUV.Y = FMath::Max( Chart.MaxUV.Y, UVs[k].Y );
			}

			FVector3f Edge1 = Positions[1] - Positions[0];
			FVector3f Edge2 = Positions[2] - Positions[0];
			FVector3f Edge3 = Positions[2] - Positions[1];

			FVector2f EdgeUV1 = UVs[1] - UVs[0];
			FVector2f EdgeUV2 = UVs[2] - UVs[0];
			FVector2f EdgeUV3 = UVs[2] - UVs[1];

			float UVArea = 0.5f * FMath::Abs(EdgeUV1.X * EdgeUV2.Y - EdgeUV1.Y * EdgeUV2.X);
			Chart.UVArea += UVArea;

			if (LayoutVersion >= ELightmapUVVersion::ScaleByEdgesLength)
			{
				float WorldLength = Edge1.Length() + Edge2.Length() + Edge3.Length();
				float UVLength = EdgeUV1.Length() + EdgeUV2.Length() + EdgeUV3.Length();

				Chart.UVLengthSum += UVLength;
				Chart.WorldLengthSum += WorldLength;
			}
			else
			{
				FVector2f UVLength;
				UVLength.X = (EdgeUV2.Y * Edge1 - EdgeUV1.Y * Edge2).Size();
				UVLength.Y = (-EdgeUV2.X * Edge1 + EdgeUV1.X * Edge2).Size();

				Chart.WorldScale += UVLength;
			}
		}

		Chart.LastTri = Tri;

#if !CHART_JOINING
		if (LayoutVersion >= ELightmapUVVersion::ScaleByEdgesLength)
		{
			if (Chart.UVLengthSum < UE_SMALL_NUMBER)
			{
				Chart.WorldScale = FVector2f(1.0f);
			}
			else
			{
				Chart.WorldScale = FVector2f(Chart.WorldLengthSum / Chart.UVLengthSum);
			}
		}
		else if (LayoutVersion >= ELightmapUVVersion::SmallChartPacking)
		{
			Chart.WorldScale /= FMath::Max(Chart.UVArea, UE_SMALL_NUMBER);
		}
		else
		{
			if (Chart.UVArea > UE_KINDA_SMALL_NUMBER)
			{
				Chart.WorldScale /= Chart.UVArea;
			}
			else
			{
				Chart.WorldScale = FVector2f::ZeroVector;
			}
		}		
#endif
	}

#if CHART_JOINING
	for( int32 i = 0; i < Charts.Num(); i++ )
	{
		FMeshChart& Chart = Charts[i];

		for( uint32 Tri = Chart.FirstTri; Tri < Chart.LastTri; Tri++ )
		{
			for( int k = 0; k < 3; k++ )
			{
				uint32 Index = 3 * SortedTris[ Tri ] + k;

				if( TranslatedMatches[ Index ] >= 0 )
				{
					checkSlow( TranslatedMatches[ TranslatedMatches[ Index ] ] == Index );

					uint32 V0i = Index;
					uint32 V0j = TranslatedMatches[ Index ];

					uint32 TriI = V0i / 3;
					uint32 TriJ = V0j / 3;
					
					if( TriJ <= TriI )
					{
						// Only need to consider one direction
						continue;
					}
					
					uint32 V1i = 3 * TriI + (V0i + 1) % 3;
					uint32 V1j = 3 * TriJ + (V0j + 1) % 3;

					int32 ChartI = i;
					int32 ChartJ = DisjointSetToChartMap[ DisjointSet[ TriJ ] ];

					FVector2f UV0i = TexCoords[ V0i ];
					FVector2f UV1i = TexCoords[ V1i ];
					FVector2f UV0j = TexCoords[ V0j ];
					FVector2f UV1j = TexCoords[ V1j ];

					FVector2f EdgeUVi = UV1i - UV0i;
					FVector2f EdgeUVj = UV1j - UV0j;

					bool bMirrored = TriangleUVArea( TriI ) * TriangleUVArea( TriJ ) < 0.0f;
					
					FVector2f EdgeOffset0 = UV0i - UV1j;
					FVector2f EdgeOffset1 = UV1i - UV0j;

					FVector2f Translation = EdgeOffset0;

					FMeshChart& ChartA = Charts[ ChartI ];
					FMeshChart& ChartB = Charts[ ChartJ ];

					for( uint32 Side = 0; Side < 4; Side++ )
					{
						// Join[] = { left, right, bottom, top }

						// FIXME
						if( bMirrored )
							continue;

						if( ChartA.Join[ Side ^ 0 ] != -1 ||
							ChartB.Join[ Side ^ 1 ] != -1 )
						{
							// Already joined with something else
							continue;
						}

						uint32 Sign = Side & 1;
						uint32 Axis = Side >> 1;

						bool bAxisAligned = FMath::Abs( EdgeUVi[ Axis ] ) < UVLAYOUT_THRESH_UVS_ARE_SAME;
						bool bBorderA = FMath::Abs( UV0i[ Axis ] - ( Sign ^ 0 ? Chart.MaxUV[ Axis ] : Chart.MinUV[ Axis ] ) ) < UVLAYOUT_THRESH_UVS_ARE_SAME;
						bool bBorderB = FMath::Abs( UV0j[ Axis ] - ( Sign ^ 1 ? Chart.MaxUV[ Axis ] : Chart.MinUV[ Axis ] ) ) < UVLAYOUT_THRESH_UVS_ARE_SAME;

						// FIXME mirrored
						if( !bAxisAligned || !bBorderA || !bBorderB )
						{
							// Edges weren't on matching rectangle borders
							continue;
						}

						FVector2f CenterA = 0.5f * ( ChartA.MinUV + ChartA.MaxUV );
						FVector2f CenterB = 0.5f * ( ChartB.MinUV + ChartB.MaxUV );

						FVector2f ExtentA = 0.5f * ( ChartA.MaxUV - ChartA.MinUV );
						FVector2f ExtentB = 0.5f * ( ChartB.MaxUV - ChartB.MinUV );

						// FIXME mirrored
						CenterB += Translation;

						FVector2f CenterDiff = CenterA - CenterB;
						FVector2f ExtentDiff = ExtentA - ExtentB;
						FVector2f Separation = ExtentA + ExtentB + CenterDiff * ( Sign ? 1.0f : -1.0f );

						bool bCenterMatch = FMath::Abs( CenterDiff[ Axis ^ 1 ] ) < UVLAYOUT_THRESH_UVS_ARE_SAME;
						bool bExtentMatch = FMath::Abs( ExtentDiff[ Axis ^ 1 ] ) < UVLAYOUT_THRESH_UVS_ARE_SAME;
						bool bSeparate    = FMath::Abs( Separation[ Axis ^ 0 ] ) < UVLAYOUT_THRESH_UVS_ARE_SAME;
	
						if( !bCenterMatch || !bExtentMatch || !bSeparate )
						{
							// Rectangles don't match up after translation
							continue;
						}

						// Found a valid edge join
						ChartA.Join[ Side ^ 0 ] = ChartJ;
						ChartB.Join[ Side ^ 1 ] = ChartI;
						break;
					}
				}
			}
		}
	}

	TArray< uint32 > JoinedSortedTris;
	JoinedSortedTris.Reserve( NumTris );

	// Detect loops
	for( uint32 Axis = 0; Axis < 2; Axis++ )
	{
		uint32 Side = Axis << 1;

		for( int32 i = 0; i < Charts.Num(); i++ )
		{
			int32 j = Charts[i].Join[ Side ^ 1 ];
			while( j != -1 )
			{
				int32 Next = Charts[j].Join[ Side ^ 1 ];
				if( Next == i )
				{
					// Break loop
					Charts[i].Join[ Side ^ 0 ] = -1;
					Charts[j].Join[ Side ^ 1 ] = -1;
					break;
				}
				j = Next;
			}
		}
	}

	// Join rows first, then columns
	for( uint32 Axis = 0; Axis < 2; Axis++ )
	{
		for( int32 i = 0; i < Charts.Num(); i++ )
		{
			FMeshChart& ChartA = Charts[i];

			if( ChartA.FirstTri == ChartA.LastTri )
			{
				// Empty chart
				continue;
			}
			
			for( uint32 Side = 0; Side < 4; Side++ )
			{
				if( ChartA.Join[ Side ] != -1 )
				{
					FMeshChart& ChartB = Charts[ ChartA.Join[ Side ] ];

					check( ChartB.Join[ Side ^ 1 ] == i );
					check( ChartB.FirstTri != ChartB.LastTri );
				}
			}
		}

		NumTris = 0;

		for( int32 i = 0; i < Charts.Num(); i++ )
		{
			FMeshChart& Chart = Charts[i];
			NumTris += Chart.LastTri - Chart.FirstTri;
		}
		check( NumTris == SortedTris.Num() );

		NumTris = 0;
		for( int32 i = 0; i < Charts.Num(); i++ )
		{
			FMeshChart& ChartA = Charts[i];

			if( ChartA.FirstTri == ChartA.LastTri )
			{
				// Empty chart
				continue;
			}

			uint32 Side = Axis << 1;

			// Find start (left, bottom)
			if( ChartA.Join[ Side ^ 0 ] == -1 )
			{
				// Add original tris
				NumTris += ChartA.LastTri - ChartA.FirstTri;

				// Continue joining until no more to the (right, top)
				int32 Next = ChartA.Join[ Side ^ 1 ];
				while( Next != -1 )
				{
					FMeshChart& ChartB = Charts[ Next ];

					NumTris += ChartB.LastTri - ChartB.FirstTri;
					Next = ChartB.Join[ Side ^ 1 ];
				}
			}
		}
		check( NumTris == SortedTris.Num() );

#if 1
		NumTris = 0;
		for( int32 i = 0; i < Charts.Num(); i++ )
		{
			FMeshChart& ChartA = Charts[i];

			if( ChartA.FirstTri == ChartA.LastTri )
			{
				// Empty chart
				continue;
			}

			// Join[] = { left, right, bottom, top }

			uint32 Side = Axis << 1;

			// Find start (left, bottom)
			if( ChartA.Join[ Side ^ 0 ] == -1 )
			{
				uint32 FirstTri = JoinedSortedTris.Num();
			
				// Add original tris
				for( uint32 Tri = ChartA.FirstTri; Tri < ChartA.LastTri; Tri++ )
				{
					JoinedSortedTris.Add( SortedTris[ Tri ] );
				}
				NumTris += ChartA.LastTri - ChartA.FirstTri;

				// Continue joining until no more to the (right, top)
				while( ChartA.Join[ Side ^ 1 ] != -1 )
				{
					FMeshChart& ChartB = Charts[ ChartA.Join[ Side ^ 1 ] ];

					check( ChartB.FirstTri != ChartB.LastTri );

					FVector2f Translation = ChartA.MinUV - ChartB.MinUV;
					Translation[ Axis ] += ChartA.MaxUV[ Axis ] - ChartA.MinUV[ Axis ];

					for( uint32 Tri = ChartB.FirstTri; Tri < ChartB.LastTri; Tri++ )
					{
						JoinedSortedTris.Add( SortedTris[ Tri ] );
						for( int k = 0; k < 3; k++ )
						{
							TexCoords[ 3 * SortedTris[ Tri ] + k ] += Translation;
						}
					}
					NumTris += ChartB.LastTri - ChartB.FirstTri;

					ChartA.Join[ Side ^ 1 ] = ChartB.Join[ Side ^ 1 ];
					ChartA.MaxUV[ Axis ] += ChartB.MaxUV[ Axis ] - ChartB.MinUV[ Axis ];
					if( LayoutVersion >= ELightmapUVVersion::ChartJoiningLFix )
					{
						// Fixing joined chart MaxUV value to properly inflate non-joined axis extent
						ChartA.MaxUV[ Axis ^ 1 ] = FMath::Max( ChartA.MaxUV[ Axis ^ 1 ], ChartA.MinUV[ Axis ^ 1 ] + ( ChartB.MaxUV[ Axis ^ 1 ] - ChartB.MinUV[ Axis ^ 1 ] ) );
					}
					ChartA.UVLengthSum += ChartB.UVLengthSum;
					ChartA.WorldLengthSum += ChartB.WorldLengthSum;
					ChartA.WorldScale += ChartB.WorldScale;
					ChartA.UVArea += ChartB.UVArea;

					ChartB.FirstTri = 0;
					ChartB.LastTri = 0;
					ChartB.UVLengthSum = 0.0f;
					ChartB.WorldLengthSum = 0.0f;
					ChartB.UVArea = 0.0f;

					DisconnectChart( Charts, ChartB, Side ^ 2 );
					DisconnectChart( Charts, ChartB, Side ^ 3 );
				}

				ChartA.FirstTri = FirstTri;
				ChartA.LastTri = JoinedSortedTris.Num();
			}
			else
			{
				// Make sure a starting chart could connect to this
				FMeshChart& ChartB = Charts[ ChartA.Join[ Side ^ 0 ] ];
				check( ChartB.Join[ Side ^ 1 ] == i );
				check( ChartB.FirstTri != ChartB.LastTri );
			}
		}
		check( NumTris == SortedTris.Num() );

		check( SortedTris.Num() == JoinedSortedTris.Num() );
		Exchange( SortedTris, JoinedSortedTris );
		JoinedSortedTris.Reset();
#endif
	}
	
	// Clean out empty charts
	for( int32 i = 0; i < Charts.Num(); )
	{
		if( Charts[i].FirstTri == Charts[i].LastTri )
		{
			Charts.RemoveAtSwap(i);
		}
		else
		{ 
			i++;
		}	
	}

	for( int32 i = 0; i < Charts.Num(); i++ )
	{
		FMeshChart& Chart = Charts[i];

		if (LayoutVersion >= ELightmapUVVersion::ScaleByEdgesLength)
		{					
			if (Chart.UVLengthSum < UE_SMALL_NUMBER)
			{
				Chart.WorldScale = FVector2f(1.0f);
			}
			else
			{
				Chart.WorldScale = FVector2f(Chart.WorldLengthSum / Chart.UVLengthSum);
			}
		}
		else
		if (LayoutVersion >= ELightmapUVVersion::SmallChartPacking)
		{
			Chart.WorldScale /= FMath::Max(Chart.UVArea, UE_SMALL_NUMBER);
		}
		else
		{
			if (Chart.UVArea > UE_KINDA_SMALL_NUMBER)
			{
				Chart.WorldScale /= Chart.UVArea;
			}
			else
			{
				Chart.WorldScale = FVector2f::ZeroVector;
			}
		}
	}
#endif

	double End = FPlatformTime::Seconds();

	UE_LOGF(LogLayoutUV, VeryVerbose, "FindCharts: %ls", *FPlatformTime::PrettyTime(End - Begin) );

	return Charts.Num();
}

#if UE_EDITOR && (UE_BUILD_DEVELOPMENT || UE_BUILD_DEBUG)
static TAutoConsoleVariable<FString> CVarLayoutUVTracePackingForInputHash(
	TEXT("LayoutUV.TracePackingForInputHash"),
	TEXT(""),
	TEXT("Activate tracing for the input hash specified in the value.\n"),
	ECVF_Default);
#endif

bool FLayoutUV::FChartPacker::FindBestPacking(const TArray< FVector2f >& TexCoords, const TArray< uint32 >& SortedTris, TArray< FMeshChart >& Charts)
{
	if( (uint32)Charts.Num() > TextureResolution * TextureResolution )
	{
		// More charts than texels
		return false;
	}
	
	TotalUVArea = 0.0f;
	for (const FMeshChart& Chart : Charts)
	{
		TotalUVArea += Chart.UVArea * Chart.WorldScale.X * Chart.WorldScale.Y;
	}

	if( TotalUVArea <= 0.0f )
	{
		return false;
	}

	uint64 StartCycles = FPlatformTime::Cycles64();
	TRACE_CPUPROFILER_EVENT_SCOPE(FChartPacker::FindBestPacking)

	// Cleanup uninitialized values to get a stable input hash
	for (FMeshChart& Chart : Charts)
	{
		Chart.PackingBias   = FVector2f::ZeroVector;
		Chart.PackingScaleU = FVector2f::ZeroVector;
		Chart.PackingScaleV = FVector2f::ZeroVector;
		Chart.UVScale       = FVector2f::ZeroVector;
	}

	FString InputHash = FMD5::HashBytes((uint8*)Charts.GetData(), Charts.Num() * Charts.GetTypeSize());

#if UE_EDITOR && (UE_BUILD_DEVELOPMENT || UE_BUILD_DEBUG)
	// When you need to find where an unexpected difference in output hash might come from
	// after changing the algorithm. You can set this CVar to activate tracing for a particular
	// input hash.
	FString InputHashTrace = CVarLayoutUVTracePackingForInputHash.GetValueOnAnyThread().TrimStartAndEnd();
	const bool bTrace = InputHashTrace.Len() > 0 && InputHash.StartsWith(InputHashTrace);
#else
	const bool bTrace = false;
#endif

	// Those might require tuning, changing them won't affect the outcome and will maintain backward compatibility
	const int32 MultithreadChartsCountThreshold       = 100*1000;
	const int32 MultithreadTextureResolutionThreshold = 1000;
	const int32 MultithreadAheadWorkCount             = 3;

	const float LinearSearchStart = 0.5f;
	const float LinearSearchStep  = 0.5f;
	const int32 BinarySearchSteps = 6;

	float UVScaleFail = TextureResolution * FMath::Sqrt( 1.0f / TotalUVArea );
	float UVScalePass = TextureResolution * FMath::Sqrt( LinearSearchStart / TotalUVArea );

	// Store successful charts packing to avoid redoing the final step
	TArray<FMeshChart>         LastPassCharts;
	TAtomic<bool>              bAbort(false);

	struct FThreadContext
	{
		TArray<FMeshChart> Charts;
		TFuture<bool>      Result;
		float              Efficiency = 0.0f;
	};

	TArray<FThreadContext> ThreadContexts;

	bool bShouldUseMultipleThreads =
		 FApp::ShouldUseThreadingForPerformance() && 
		!bTrace &&
 		 Charts.Num() >= MultithreadChartsCountThreshold &&
		 TextureResolution >= MultithreadTextureResolutionThreshold;

	if ( bShouldUseMultipleThreads )
	{
		// Do forward work only when multi-thread activated
		ThreadContexts.SetNum(MultithreadAheadWorkCount);
	}

	// Linear search for first fit
	float LastEfficiency = 0.0f;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(LinearSearch);

		while(!bAbort)
		{
			// Launch forward work in other threads
			for (int32 Index = 0; Index < ThreadContexts.Num(); ++Index)
			{
				ThreadContexts[Index].Charts = Charts;
				float ThreadUVScale  = UVScalePass * FMath::Pow(LinearSearchStep, Index + 1);
				ThreadContexts[Index].Result =
					Async(
						EAsyncExecution::ThreadPool,
						[this, &ThreadContexts, &SortedTris, &TexCoords, &bAbort, ThreadUVScale, Index]()
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(SearchStep);
							return PackCharts(ThreadContexts[Index].Charts, ThreadUVScale, TexCoords, SortedTris, ThreadContexts[Index].Efficiency, bAbort, false);
						}
					);
			}

			if (bTrace)
			{
				UE_LOGF(LogLayoutUV, Log, "[LAYOUTUV_TRACE] Scale %f", UVScalePass);
			}

			// Process the first iteration in this thread
			bool bFit = false;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(SearchStep);
				bFit = PackCharts(Charts, UVScalePass, TexCoords, SortedTris, LastEfficiency, bAbort, bTrace);
			}

			// Wait for the work sequentially and cancel everything once we have a first viable solution
			for (int32 Index = 0; Index < ThreadContexts.Num() + 1; ++Index)
			{
				// The first result is not coming from a future
				bFit = Index == 0 ? bFit : ThreadContexts[Index - 1].Result.Get();
				if (bFit && !bAbort)
				{
					// We got a success, cancel other searches
					bAbort = true;

					if (Index > 0)
					{
						Charts         = ThreadContexts[Index - 1].Charts;
						LastEfficiency = ThreadContexts[Index - 1].Efficiency;
					}

					LastPassCharts = Charts;
				}

				if (!bAbort)
				{
					UVScaleFail = UVScalePass;
					UVScalePass *= LinearSearchStep;
				}
			}
		}
	}

	// Binary search for best fit
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BinarySearch);

		bAbort = false;
		for( int32 i = 0; i < BinarySearchSteps; i++ )
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SearchStep);

			float UVScale = 0.5f * ( UVScaleFail + UVScalePass );

			if (bTrace)
			{
				UE_LOGF(LogLayoutUV, Log, "[LAYOUTUV_TRACE] Scale %f", UVScale);
			}

			float Efficiency = 0.0f;
			bool bFit = PackCharts(Charts, UVScale, TexCoords, SortedTris, Efficiency, bAbort, bTrace);
			if( bFit )
			{
				LastPassCharts = Charts;
				float EfficiencyGainPercent = 100.0f * FMath::Abs(Efficiency - LastEfficiency);
				LastEfficiency = Efficiency;

				// Early out when we're inside a 1% efficiency range
				if (LayoutVersion >= ELightmapUVVersion::Segments2D && EfficiencyGainPercent <= 1.0f)
				{
					break;
				}
			
				UVScalePass = UVScale;
			}
			else
			{
				UVScaleFail = UVScale;
			}
		}
	}

	if (LayoutVersion < ELightmapUVVersion::ScaleChartsOrderingFix)
	{
		// Early versions applied a sort that was determinist
		// but dependent on earlier sorts. Since we strive to maintain
		// backward compatibility of the UV layout to avoid screwing
		// with already backed static lighting, we must apply a final
		// scaling and packing that will reuse the last step's ordering
		// whether it was a failure or not.
		PackCharts(Charts, UVScalePass, TexCoords, SortedTris, LastEfficiency, bAbort, bTrace);
	}
	else
	{
		// In case the last step was a failure, restore from last known good computation
		Charts = LastPassCharts;
	}

	FString OutputHash = FMD5::HashBytes((uint8*)Charts.GetData(), Charts.Num() * Charts.GetTypeSize());

	// Increase verbosity level to use this for packing results validation when modifying code
	UE_LOGF(LogLayoutUV, Verbose, "FindBestPacking (Input Data MD5: %ls, Output Data MD5: %ls, LayoutVersion: %d, Efficiency: %0.2f %%)", *InputHash, *OutputHash, int(LayoutVersion), LastEfficiency*100);

	static TAtomic<uint64> Count(0);
	static TAtomic<uint64> TotalCycles(0);
	static TAtomic<uint64> Efficiency(0);

	FindBestPackingCount++;
	FindBestPackingEfficiency += LastEfficiency*100000;
	FindBestPackingCycles += FPlatformTime::Cycles64() - StartCycles;

	return true;
}

void FLayoutUV::ResetStats()
{
	FindBestPackingCount = 0;
	FindBestPackingEfficiency = 0;
	FindBestPackingCycles = 0;
}

void FLayoutUV::LogStats()
{
	UE_LOGF(LogLayoutUV, Log, "FindBestPacking (Total Time: %ls, Avg Efficiency: %f)", *FPlatformTime::PrettyTime(FPlatformTime::ToSeconds64(FindBestPackingCycles.Load())), double(FindBestPackingEfficiency.Load()) / (FindBestPackingCount.Load()*1000));
}

void FLayoutUV::FChartPacker::ScaleCharts( TArray< FMeshChart >& Charts, float UVScale )
{
	for( int32 i = 0; i < Charts.Num(); i++ )
	{
		FMeshChart& Chart = Charts[i];
		Chart.UVScale = Chart.WorldScale * UVScale;
	}
	
	if ( LayoutVersion >= ELightmapUVVersion::ScaleChartsOrderingFix )
	{
		// Unsort the charts to make sure ScaleCharts always return the same ordering
		Algo::IntroSort( Charts, []( const FMeshChart& A, const FMeshChart& B )
		{
			return A.Id < B.Id;
		});
	}

	// Scale charts such that they all fit and roughly total the same area as before
#if 1
	float UniformScale = 1.0f;
	for( int i = 0; i < 1000; i++ )
	{
		uint32 NumMaxedOut = 0;
		float ScaledUVArea = 0.0f;
		for( int32 ChartIndex = 0; ChartIndex < Charts.Num(); ChartIndex++ )
		{
			FMeshChart& Chart = Charts[ChartIndex];

			FVector2f ChartSize	= Chart.MaxUV - Chart.MinUV;
			FVector2f ChartSizeScaled = ChartSize * Chart.UVScale * UniformScale;

			const float MaxChartEdge = TextureResolution - 1.0f;
			const float LongestChartEdge = FMath::Max( ChartSizeScaled.X, ChartSizeScaled.Y );

			const float Epsilon = 0.01f;
			if( LongestChartEdge + Epsilon > MaxChartEdge )
			{
				// Rescale oversized charts to fit
				Chart.UVScale.X = MaxChartEdge / FMath::Max( ChartSize.X, ChartSize.Y );
				Chart.UVScale.Y = MaxChartEdge / FMath::Max( ChartSize.X, ChartSize.Y );
				NumMaxedOut++;
			}
			else
			{
				Chart.UVScale.X *= UniformScale;
				Chart.UVScale.Y *= UniformScale;
			}
			
			ScaledUVArea += Chart.UVArea * Chart.UVScale.X * Chart.UVScale.Y;
		}

		if( NumMaxedOut == 0 )
		{
			// No charts maxed out so no need to rebalance
			break;
		}

		if( NumMaxedOut == Charts.Num() )
		{
			// All charts are maxed out
			break;
		}

		// Scale up smaller charts to maintain expected total area
		// Want ScaledUVArea == TotalUVArea * UVScale^2
		float RebalanceScale = UVScale * FMath::Sqrt( TotalUVArea / ScaledUVArea );
		if( RebalanceScale < 1.01f )
		{
			// Stop if further rebalancing is minor
			break;
		}
		UniformScale = RebalanceScale;
	}
#endif

#if 1
	float NonuniformScale = 1.0f;
	for( int i = 0; i < 1000; i++ )
	{
		uint32 NumMaxedOut = 0;
		float ScaledUVArea = 0.0f;
		for( int32 ChartIndex = 0; ChartIndex < Charts.Num(); ChartIndex++ )
		{
			FMeshChart& Chart = Charts[ChartIndex];

			for( int k = 0; k < 2; k++ )
			{
				const float MaximumChartSize = TextureResolution - 1.0f;
				const float ChartSize = Chart.MaxUV[k] - Chart.MinUV[k];
				const float ChartSizeScaled = ChartSize * Chart.UVScale[k] * NonuniformScale;

				const float Epsilon = 0.01f;
				if( ChartSizeScaled + Epsilon > MaximumChartSize )
				{
					// Scale oversized charts to max size
					Chart.UVScale[k] = MaximumChartSize / ChartSize;
					NumMaxedOut++;
				}
				else
				{
					Chart.UVScale[k] *= NonuniformScale;
				}
			}

			ScaledUVArea += Chart.UVArea * Chart.UVScale.X * Chart.UVScale.Y;
		}

		if( NumMaxedOut == 0 )
		{
			// No charts maxed out so no need to rebalance
			break;
		}

		if( NumMaxedOut == Charts.Num() * 2 )
		{
			// All charts are maxed out in both dimensions
			break;
		}

		// Scale up smaller charts to maintain expected total area
		// Want ScaledUVArea == TotalUVArea * UVScale^2
		float RebalanceScale = UVScale * FMath::Sqrt( TotalUVArea / ScaledUVArea );
		if( RebalanceScale < 1.01f )
		{
			// Stop if further rebalancing is minor
			break;
		}
		NonuniformScale = RebalanceScale;
	}
#endif

	// Sort charts from largest to smallest
	struct FCompareCharts
	{
		FORCEINLINE bool operator()( const FMeshChart& A, const FMeshChart& B ) const
		{
			// Rect area
			FVector2f ChartRectA = ( A.MaxUV - A.MinUV ) * A.UVScale;
			FVector2f ChartRectB = ( B.MaxUV - B.MinUV ) * B.UVScale;
			return ChartRectA.X * ChartRectA.Y > ChartRectB.X * ChartRectB.Y;
		}
	};
	Algo::IntroSort( Charts, FCompareCharts() );
}

// Hash function to use FMD5Hash in TMap
inline uint32 GetTypeHash(const FMD5Hash& Hash)
{
	uint32* HashAsInt32 = (uint32*)Hash.GetBytes();
	return HashAsInt32[0] ^ HashAsInt32[1] ^ HashAsInt32[2] ^ HashAsInt32[3];
}

bool FLayoutUV::FChartPacker::PackCharts(TArray< FMeshChart >& Charts, float UVScale, const TArray< FVector2f >& TexCoords, const TArray< uint32 >& SortedTris, float& OutEfficiency, TAtomic<bool>& bAbort, bool bTrace)
{
	ScaleCharts( Charts, UVScale );
	TRACE_CPUPROFILER_EVENT_SCOPE(FChartPacker::PackCharts)

	FAllocator2D BestChartRaster(FAllocator2D::EMode::UsedSegments, TextureResolution, TextureResolution, LayoutVersion);
	FAllocator2D ChartRaster    (FAllocator2D::EMode::UsedSegments, TextureResolution, TextureResolution, LayoutVersion);
	FAllocator2D LayoutRaster   (FAllocator2D::EMode::FreeSegments, TextureResolution, TextureResolution, LayoutVersion);

	uint64 RasterizeCycles = 0;
	uint64 FindCycles = 0;

	double BeginPackCharts = FPlatformTime::Seconds();

	OutEfficiency = 0.0f;
	LayoutRaster.Clear();

	// Store the position where we found a spot for each unique raster
	// so we can skip whole sections we know won't work out.
	// This method is obviously more efficient with smaller charts
	// but helps tremendously as the number of charts goes up for
	// the same texture space. This helps counteract the slowdown
	// induced by having more parts to place in the grid and is
	// particularly useful for foliage.
	TMap<FMD5Hash, FVector2f> BestStartPos;

	// Reduce Insights CPU tracing to once per batch
	const int32 BatchSize = 1024;
	for( int32 ChartIndex = 0; ChartIndex < Charts.Num() && !bAbort.Load(EMemoryOrder::Relaxed);)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChartBatch);
		for( int32 BatchIndex = 0; BatchIndex < BatchSize && ChartIndex < Charts.Num() && !bAbort.Load(EMemoryOrder::Relaxed); ++ChartIndex, ++BatchIndex)
		{
			FMeshChart& Chart = Charts[ChartIndex];

			// Try different orientations and pick best
			int32				BestOrientation = -1;
			FAllocator2D::FRect	BestRect = { ~0u, ~0u, ~0u, ~0u };

			// Refactored BestRect comparison code in one place so this can be customized per version if needed
			TFunction<bool (const FAllocator2D::FRect&)> IsBestRect;
			if ( LayoutVersion >= ELightmapUVVersion::OptimalSurfaceArea )
			{
				// This version focus on minimal surface area giving fairness to both horizontal and vertical chart placement
				// instead of only taking the pixel offset of the lower left corner into account.
				IsBestRect = 
					[&BestRect](const FAllocator2D::FRect& Rect)
					{
						return ((Rect.X+Rect.W) + (Rect.Y+Rect.H)) < ((BestRect.X+BestRect.W) + (BestRect.Y+BestRect.H));
					};
			}
			else
			{
				IsBestRect = 
					[this, &BestRect](const FAllocator2D::FRect& Rect)
					{
						return Rect.X + Rect.Y * TextureResolution < BestRect.X + BestRect.Y * TextureResolution;
					};
			}

			for( int32 Orientation = 0; Orientation < 8; Orientation++ )
			{
				// TODO If any dimension is less than 1 pixel shrink dimension to zero

				OrientChart( Chart, Orientation);
			
				FVector2f ChartSize = Chart.MaxUV - Chart.MinUV;
				ChartSize = ChartSize.X * Chart.PackingScaleU + ChartSize.Y * Chart.PackingScaleV;

				// Only need half pixel dilate for rects
				FAllocator2D::FRect	Rect;
				Rect.X = 0;
				Rect.Y = 0;
				Rect.W = FMath::CeilToInt( FMath::Abs( ChartSize.X ) + 1.0f );
				Rect.H = FMath::CeilToInt( FMath::Abs( ChartSize.Y ) + 1.0f );

				// Just in case lack of precision pushes it over
				Rect.W = FMath::Min( TextureResolution, Rect.W );
				Rect.H = FMath::Min( TextureResolution, Rect.H );

				const bool bRectPack = false;

				if( bRectPack )
				{
					if( LayoutRaster.Find( Rect ) )
					{
						if( IsBestRect(Rect) )
						{
							BestOrientation = Orientation;
							BestRect = Rect;
						}
					}
					else
					{
						continue;
					}
				}
				else
				{
					if ( LayoutVersion >= ELightmapUVVersion::Segments && Orientation % 4 == 1 )
					{
						ChartRaster.FlipX( Rect );
					}
					else if ( LayoutVersion >= ELightmapUVVersion::Segments && Orientation % 4 == 3 )
					{
						ChartRaster.FlipY( Rect );
					}
					else
					{
						int32 BeginRasterize = FPlatformTime::Cycles();
						RasterizeChart( Chart, TexCoords, SortedTris, Rect.W , Rect.H, ChartRaster);
						RasterizeCycles += FPlatformTime::Cycles() - BeginRasterize;
					}

					bool bFound = false;

					uint32 BeginFind = FPlatformTime::Cycles();
					if ( LayoutVersion == ELightmapUVVersion::BitByBit )
					{
						bFound = LayoutRaster.FindBitByBit( Rect, ChartRaster );
					}
					else if ( LayoutVersion >= ELightmapUVVersion::Segments )
					{
						// Use the real raster size for optimal placement
						FAllocator2D::FRect RasterRect = Rect;
						RasterRect.W = ChartRaster.GetRasterWidth();
						RasterRect.H = ChartRaster.GetRasterHeight();

						// Nothing rasterized, returning 0,0 as fast as possible
						// since this is what the actual algorithm is doing but
						// we might have to flag the entire UV map as invalid since
						// charts are going to overlap
						if (RasterRect.H == 0 && RasterRect.W == 0)
						{
							Rect.X = 0;
							Rect.Y = 0;
							bFound = true;
						}
						else
						{
							FMD5Hash RasterMD5 = ChartRaster.GetRasterMD5();
							FVector2f* StartPos = BestStartPos.Find(RasterMD5);

							if (StartPos)
							{
								RasterRect.X = StartPos->X;
								RasterRect.Y = StartPos->Y;
							}

							LayoutRaster.ResetStats();
							bFound = LayoutRaster.FindWithSegments(RasterRect, ChartRaster, IsBestRect);
							if (bFound)
							{
								// Store only the best possible position in the hash table so we can start from there for other identical charts
								BestStartPos.Add(RasterMD5, FVector2f(RasterRect.X, RasterRect.Y));

								// Since the older version stops searching at Width - Rect.W instead of using the raster size,
								// it means a perfect rasterized square of 2,2 won't fit a 2,2 hole at the end of a row if Rect.W = 3.
								// Because of that, we have no choice to worsen our algorithm behavior for backward compatibility.

								// Once we know the best possible position, we'll continue our search from there with the original
								// rect value if it differs from the raster rect to ensure we get the same result as the old algorithm.
								if (LayoutVersion < ELightmapUVVersion::Segments2D && (Rect.X != RasterRect.X || Rect.Y != RasterRect.Y))
								{
									Rect.X = RasterRect.X;
									Rect.Y = RasterRect.Y;

									bFound = LayoutRaster.FindWithSegments(Rect, ChartRaster, IsBestRect);
								}
								else
								{
									// We can't copy W and H here as they might be different than what we got initially
									Rect.X = RasterRect.X;
									Rect.Y = RasterRect.Y;
								}
							}

							LayoutRaster.PublishStats(ChartIndex, Orientation, bFound, Rect, BestRect, RasterMD5, IsBestRect);
						}

					}
					FindCycles += FPlatformTime::Cycles() - BeginFind;

					if (bTrace)
					{
						UE_LOGF(LogLayoutUV, Log, "[LAYOUTUV_TRACE] Chart %d Orientation %d Found = %d Rect = %d,%d,%d,%d\n", ChartIndex, Orientation, bFound ? 1 : 0, Rect.X, Rect.Y, Rect.W, Rect.H);
					}

					if( bFound )
					{
						if( IsBestRect(Rect) )
						{
							BestChartRaster = ChartRaster;

							BestOrientation = Orientation;
							BestRect = Rect;

							if ( BestRect.X == 0 && BestRect.Y == 0 )
							{
								// BestRect can't be beat, stop here
								break;
							}
						}
					}
					else
					{
						continue;
					}
				}
			}

			if( BestOrientation >= 0 )
			{
				// Add chart to layout
				OrientChart( Chart, BestOrientation );

				LayoutRaster.Alloc( BestRect, BestChartRaster );

				Chart.PackingBias.X += BestRect.X;
				Chart.PackingBias.Y += BestRect.Y;
			}
			else
			{
				if (bTrace)
				{
					UE_LOGF(LogLayoutUV, Log, "[LAYOUTUV_TRACE] Chart %d Found no orientation that fit\n", ChartIndex);
				}
				// Found no orientation that fit
				return false;
			}
		}
	}

	if (bAbort)
	{
		return false;
	}

	const uint32 TotalTexels = TextureResolution * TextureResolution;
	const uint32 UsedTexels  = LayoutRaster.GetUsedTexels();

	OutEfficiency = float( UsedTexels ) / TotalTexels;
	double EndPackCharts = FPlatformTime::Seconds();

	UE_LOGF(LogLayoutUV, VeryVerbose, "PackCharts: %ls",	*FPlatformTime::PrettyTime(EndPackCharts - BeginPackCharts));
	UE_LOGF(LogLayoutUV, VeryVerbose, "  Rasterize: %llu",	RasterizeCycles);
	UE_LOGF(LogLayoutUV, VeryVerbose, "  Find: %llu",		FindCycles);

	return true;
}

void FLayoutUV::FChartPacker::OrientChart( FMeshChart& Chart, int32 Orientation )
{
	switch( Orientation )
	{
	case 0:
		// 0 degrees
		Chart.PackingScaleU = FVector2f( Chart.UVScale.X, 0 );
		Chart.PackingScaleV = FVector2f( 0, Chart.UVScale.Y );
		Chart.PackingBias = -Chart.MinUV.X * Chart.PackingScaleU - Chart.MinUV.Y * Chart.PackingScaleV + 0.5f;
		break;
	case 1:
		// 0 degrees, flip x
		Chart.PackingScaleU = FVector2f( -Chart.UVScale.X, 0 );
		Chart.PackingScaleV = FVector2f( 0, Chart.UVScale.Y );
		Chart.PackingBias = -Chart.MaxUV.X * Chart.PackingScaleU - Chart.MinUV.Y * Chart.PackingScaleV + 0.5f;
		break;
	case 2:
		// 90 degrees
		Chart.PackingScaleU = FVector2f( 0, -Chart.UVScale.X );
		Chart.PackingScaleV = FVector2f( Chart.UVScale.Y, 0 );
		Chart.PackingBias = -Chart.MaxUV.X * Chart.PackingScaleU - Chart.MinUV.Y * Chart.PackingScaleV + 0.5f;
		break;
	case 3:
		// 90 degrees, flip x
		Chart.PackingScaleU = FVector2f( 0, Chart.UVScale.X );
		Chart.PackingScaleV = FVector2f( Chart.UVScale.Y, 0 );
		Chart.PackingBias = -Chart.MinUV.X * Chart.PackingScaleU - Chart.MinUV.Y * Chart.PackingScaleV + 0.5f;
		break;
	case 4:
		// 180 degrees
		Chart.PackingScaleU = FVector2f( -Chart.UVScale.X, 0 );
		Chart.PackingScaleV = FVector2f( 0, -Chart.UVScale.Y );
		Chart.PackingBias = -Chart.MaxUV.X * Chart.PackingScaleU - Chart.MaxUV.Y * Chart.PackingScaleV + 0.5f;
		break;
	case 5:
		// 180 degrees, flip x
		Chart.PackingScaleU = FVector2f( Chart.UVScale.X, 0 );
		Chart.PackingScaleV = FVector2f( 0, -Chart.UVScale.Y );
		Chart.PackingBias = -Chart.MinUV.X * Chart.PackingScaleU - Chart.MaxUV.Y * Chart.PackingScaleV + 0.5f;
		break;
	case 6:
		// 270 degrees
		Chart.PackingScaleU = FVector2f( 0, Chart.UVScale.X );
		Chart.PackingScaleV = FVector2f( -Chart.UVScale.Y, 0 );
		Chart.PackingBias = -Chart.MinUV.X * Chart.PackingScaleU - Chart.MaxUV.Y * Chart.PackingScaleV + 0.5f;
		break;
	case 7:
		// 270 degrees, flip x
		Chart.PackingScaleU = FVector2f( 0, -Chart.UVScale.X );
		Chart.PackingScaleV = FVector2f( -Chart.UVScale.Y, 0 );
		Chart.PackingBias = -Chart.MaxUV.X * Chart.PackingScaleU - Chart.MaxUV.Y * Chart.PackingScaleV + 0.5f;
		break;
	}
}

// Max of 2048x2048 due to precision
// Dilate in 28.4 fixed point. Half pixel dilation is conservative rasterization.
// Dilation same as Minkowski sum of triangle and square.
template< int32 Dilate >
void RasterizeTriangle( FAllocator2D& Shader, const FVector2f Points[3], int32 ScissorWidth, int32 ScissorHeight )
{
	const FVector2f HalfPixel( 0.5f, 0.5f );
	FVector2f p0 = Points[0] - HalfPixel;
	FVector2f p1 = Points[1] - HalfPixel;
	FVector2f p2 = Points[2] - HalfPixel;

	// Correct winding
	float Facing = ( p0.X - p1.X ) * ( p2.Y - p0.Y ) - ( p0.Y - p1.Y ) * ( p2.X - p0.X );
	if( Facing < 0.0f )
	{
		Swap( p0, p2 );
	}

	// 28.4 fixed point
	const int32 X0 = (int32)( 16.0f * p0.X + 0.5f );
	const int32 X1 = (int32)( 16.0f * p1.X + 0.5f );
	const int32 X2 = (int32)( 16.0f * p2.X + 0.5f );
	
	const int32 Y0 = (int32)( 16.0f * p0.Y + 0.5f );
	const int32 Y1 = (int32)( 16.0f * p1.Y + 0.5f );
	const int32 Y2 = (int32)( 16.0f * p2.Y + 0.5f );

	// Bounding rect
	int32 MinX = ( FMath::Min3( X0, X1, X2 ) - Dilate + 15 ) / 16;
	int32 MaxX = ( FMath::Max3( X0, X1, X2 ) + Dilate + 15 ) / 16;
	int32 MinY = ( FMath::Min3( Y0, Y1, Y2 ) - Dilate + 15 ) / 16;
	int32 MaxY = ( FMath::Max3( Y0, Y1, Y2 ) + Dilate + 15 ) / 16;

	// Clip to image
	MinX = FMath::Clamp( MinX, 0, ScissorWidth );
	MaxX = FMath::Clamp( MaxX, 0, ScissorWidth );
	MinY = FMath::Clamp( MinY, 0, ScissorHeight );
	MaxY = FMath::Clamp( MaxY, 0, ScissorHeight );

	// Deltas
	const int32 DX01 = X0 - X1;
	const int32 DX12 = X1 - X2;
	const int32 DX20 = X2 - X0;

	const int32 DY01 = Y0 - Y1;
	const int32 DY12 = Y1 - Y2;
	const int32 DY20 = Y2 - Y0;

	// Half-edge constants
	int32 C0 = DY01 * X0 - DX01 * Y0;
	int32 C1 = DY12 * X1 - DX12 * Y1;
	int32 C2 = DY20 * X2 - DX20 * Y2;

	// Correct for fill convention
	C0 += ( DY01 < 0 || ( DY01 == 0 && DX01 > 0 ) ) ? 0 : -1;
	C1 += ( DY12 < 0 || ( DY12 == 0 && DX12 > 0 ) ) ? 0 : -1;
	C2 += ( DY20 < 0 || ( DY20 == 0 && DX20 > 0 ) ) ? 0 : -1;

	// Dilate edges
	C0 += ( abs(DX01) + abs(DY01) ) * Dilate;
	C1 += ( abs(DX12) + abs(DY12) ) * Dilate;
	C2 += ( abs(DX20) + abs(DY20) ) * Dilate;

	for( int32 y = MinY; y < MaxY; y++ )
	{
		for( int32 x = MinX; x < MaxX; x++ )
		{
			// same as Edge1 >= 0 && Edge2 >= 0 && Edge3 >= 0
			int32 IsInside;
			IsInside  = C0 + (DX01 * y - DY01 * x) * 16;
			IsInside |= C1 + (DX12 * y - DY12 * x) * 16;
			IsInside |= C2 + (DX20 * y - DY20 * x) * 16;

			if( IsInside >= 0 )
			{
				Shader.SetBit( x, y );
			}
		}
	}
}

void FLayoutUV::FChartPacker::RasterizeChart( const FMeshChart& Chart, const TArray< FVector2f >& TexCoords, const TArray< uint32 >& SortedTris, uint32 RectW, uint32 RectH, FAllocator2D& OutChartRaster )
{
	// Bilinear footprint is -1 to 1 pixels. If packed geometrically, only a half pixel dilation
	// would be needed to guarantee all charts were at least 1 pixel away, safe for bilinear filtering.
	// Unfortunately, with pixel packing a full 1 pixel dilation is required unless chart edges exactly
	// align with pixel centers.

	OutChartRaster.Clear();

	for( uint32 Tri = Chart.FirstTri; Tri < Chart.LastTri; Tri++ )
	{
		FVector2f Points[3];
		for ( int k = 0; k < 3; k++ )
		{
			const FVector2f& UV = TexCoords[ 3 * SortedTris[ Tri ] + k ];
			Points[k] = UV.X * Chart.PackingScaleU + UV.Y * Chart.PackingScaleV + Chart.PackingBias;
		}

		RasterizeTriangle< 16 >( OutChartRaster, Points, RectW, RectH );
	}

	if ( LayoutVersion >= ELightmapUVVersion::Segments )
	{
		OutChartRaster.CreateUsedSegments();
	}
}

bool FLayoutUV::FindBestPacking(uint32 InTextureResolution)
{
	FChartPacker Packer(MeshView, LayoutVersion, InTextureResolution);
	bool bPackingFound = Packer.FindBestPacking(MeshTexCoords, MeshSortedTris, MeshCharts);
	PackedTextureResolution = bPackingFound ? InTextureResolution : 0;
	return bPackingFound;
}

void FLayoutUV::CommitPackedUVs()
{
	if (PackedTextureResolution == 0)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FLayoutUV::CommitPackedUVs)

	// Alloc new UV channel
	MeshView.InitOutputTexcoords(MeshTexCoords.Num());

	// Commit chart UVs
	for( int32 i = 0; i < MeshCharts.Num(); i++ )
	{
		FMeshChart& Chart = MeshCharts[i];

		Chart.PackingScaleU /= PackedTextureResolution;
		Chart.PackingScaleV /= PackedTextureResolution;
		Chart.PackingBias /= PackedTextureResolution;

		for( uint32 Tri = Chart.FirstTri; Tri < Chart.LastTri; Tri++ )
		{
			for( int k = 0; k < 3; k++ )
			{
				uint32 Index = 3 * MeshSortedTris[ Tri ] + k;
				const FVector2f& UV = MeshTexCoords[ Index ];
				FVector2f TransformedUV = UV.X * Chart.PackingScaleU + UV.Y * Chart.PackingScaleV + Chart.PackingBias;
				MeshView.SetOutputTexcoord(Index, TransformedUV);
			}
		}
	}
}

inline bool FLayoutUV::FChartFinder::PositionsMatch( uint32 a, uint32 b ) const
{
	return ( MeshView.GetPosition(a) - MeshView.GetPosition(b) ).IsNearlyZero( THRESH_POINTS_ARE_SAME );
}

inline bool FLayoutUV::FChartFinder::NormalsMatch( uint32 a, uint32 b ) const
{
	return ( MeshView.GetNormal(a) - MeshView.GetNormal(b) ).IsNearlyZero( THRESH_NORMALS_ARE_SAME );
}

inline bool FLayoutUV::FChartFinder::UVsMatch( uint32 a, uint32 b ) const
{
	return ( MeshView.GetInputTexcoord(a) - MeshView.GetInputTexcoord(b) ).IsNearlyZero(UVLAYOUT_THRESH_UVS_ARE_SAME);
}

inline bool FLayoutUV::FChartFinder::VertsMatch( uint32 a, uint32 b ) const
{
	return PositionsMatch( a, b ) && UVsMatch( a, b );
}

// Signed UV area
inline float FLayoutUV::FChartFinder::TriangleUVArea( uint32 Tri ) const
{
	FVector2f UVs[3];
	for( int k = 0; k < 3; k++ )
	{
		UVs[k] = MeshView.GetInputTexcoord(3 * Tri + k);
	}

	FVector2f EdgeUV1 = UVs[1] - UVs[0];
	FVector2f EdgeUV2 = UVs[2] - UVs[0];
	return 0.5f * ( EdgeUV1.X * EdgeUV2.Y - EdgeUV1.Y * EdgeUV2.X );
}

inline void FLayoutUV::FChartFinder::DisconnectChart( TArray< FMeshChart >& Charts, FMeshChart& Chart, uint32 Side )
{
	if( Chart.Join[ Side ] != -1 )
	{
		Charts[ Chart.Join[ Side ] ].Join[ Side ^ 1 ] = -1;
		Chart.Join[ Side ] = -1;
	}
}

inline float FLayoutUV::FChartFinder::GetUVEqualityThreshold() const
{
	return LayoutVersion >= ELightmapUVVersion::SmallChartPacking ? NEW_UVS_ARE_SAME : LEGACY_UVS_ARE_SAME;
}
