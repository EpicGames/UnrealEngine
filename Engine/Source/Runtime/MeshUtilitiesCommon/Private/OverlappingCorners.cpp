// Copyright Epic Games, Inc. All Rights Reserved.

#include "OverlappingCorners.h"
#include "MeshUtilitiesCommon.h"

// Create a mesh view from arrays of vertices & indices
struct FOverlappingCornersArrayMeshView : FLayoutUV::IMeshView
{
	const TArray<FVector3f>& Vertices;
	const TArray<uint32>& Indices;

	FOverlappingCornersArrayMeshView(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices)
		: Vertices(InVertices)
		, Indices(InIndices)
	{
	}

	// IMeshView Interface --
	virtual uint32      GetNumIndices() const override { return Indices.Num(); }
	virtual FVector3f   GetPosition(uint32 Index) const override { return Vertices[Indices[Index]]; }

	virtual FVector3f   GetNormal(uint32 Index) const override { check(0); return FVector3f(); }
	virtual FVector2f   GetInputTexcoord(uint32 Index) const override { check(0); return FVector2f(); }
	virtual void        InitOutputTexcoords(uint32 Num) override { check(0); }
	virtual void        SetOutputTexcoord(uint32 Index, const FVector2f& Value) override { check(0); }
	// IMeshView Interface --
};

FOverlappingCorners::FOverlappingCorners(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, float ComparisonThreshold)
	: FOverlappingCorners(FOverlappingCornersArrayMeshView(InVertices, InIndices), ComparisonThreshold)
{
}

FOverlappingCorners::FOverlappingCorners(const FLayoutUV::IMeshView& MeshView, float ComparisonThreshold)
{
	const int32 NumWedges = MeshView.GetNumIndices();

	// Create a list of vertex Z/index pairs
	TArray<FIndexAndZ> VertIndexAndZ;
	VertIndexAndZ.Reserve(NumWedges);
	for (int32 WedgeIndex = 0; WedgeIndex < NumWedges; WedgeIndex++)
	{
		VertIndexAndZ.Emplace(WedgeIndex, MeshView.GetPosition(WedgeIndex));
	}

	// Sort the vertices by z value
	VertIndexAndZ.Sort(FCompareIndexAndZ());

	Init(NumWedges);

	// Search for duplicates, quickly!
	for (int32 i = 0; i < VertIndexAndZ.Num(); i++)
	{
		// only need to search forward, since we add pairs both ways
		for (int32 j = i + 1; j < VertIndexAndZ.Num(); j++)
		{
			if (FMath::Abs(VertIndexAndZ[j].Z - VertIndexAndZ[i].Z) > ComparisonThreshold)
				break; // can't be any more dups

			const FVector3f& PositionA = MeshView.GetPosition(VertIndexAndZ[i].Index);
			const FVector3f& PositionB = MeshView.GetPosition(VertIndexAndZ[j].Index);

			if (PointsEqual(PositionA, PositionB, ComparisonThreshold))
			{
				Add(VertIndexAndZ[i].Index, VertIndexAndZ[j].Index);
			}
		}
	}

	FinishAdding();
}

void FOverlappingCorners::Init(int32 NumIndices)
{
	Arrays.Reset();
	Sets.Reset();
	bFinishedAdding = false;

	IndexBelongsTo.Reset(NumIndices);
	IndexBelongsTo.AddUninitialized(NumIndices);
	FMemory::Memset(IndexBelongsTo.GetData(), 0xFF, NumIndices * sizeof(int32));
}

void FOverlappingCorners::Add(int32 Key, int32 Value)
{
	check(Key != Value);
	check(bFinishedAdding == false);

	int32 ContainerIndex = IndexBelongsTo[Key];
	if (ContainerIndex == INDEX_NONE)
	{
		ContainerIndex = Arrays.Num();
		TArray<int32>& Container = Arrays.AddDefaulted_GetRef();
		Container.Reserve(6);
		Container.Add(Key);
		Container.Add(Value);
		IndexBelongsTo[Key] = ContainerIndex;
		IndexBelongsTo[Value] = ContainerIndex;
	}
	else
	{
		IndexBelongsTo[Value] = ContainerIndex;

		TArray<int32>& ArrayContainer = Arrays[ContainerIndex];
		if (ArrayContainer.Num() == 1)
		{
			// Container is a set
			Sets[ArrayContainer.Last()].Add(Value);
		}
		else
		{
			// Container is an array
			ArrayContainer.AddUnique(Value);

			// Change container into set when one vertex is shared by large number of triangles
			if (ArrayContainer.Num() > 12)
			{
				int32 SetIndex = Sets.Num();
				TSet<int32>& Set = Sets.AddDefaulted_GetRef();
				Set.Append(ArrayContainer);

				// Having one element means we are using a set
				// An array will never have just 1 element normally because we add them as pairs
				ArrayContainer.Reset(1);
				ArrayContainer.Add(SetIndex);
			}
		}
	}
}

void FOverlappingCorners::FinishAdding()
{
	check(bFinishedAdding == false);

	for (TArray<int32>& Array : Arrays)
	{
		// Turn sets back into arrays for easier iteration code
		// Also reduces peak memory later in the import process
		if (Array.Num() == 1)
		{
			TSet<int32>& Set = Sets[Array.Last()];
			Array.Reset(Set.Num());
			for (int32 i : Set)
			{
				Array.Add(i);
			}
		}

		// Sort arrays now to avoid sort multiple times
		Array.Sort();
	}

	Sets.Empty();

	bFinishedAdding = true;
}

uint32 FOverlappingCorners::GetAllocatedSize(void) const
{
	uint32 BaseMemoryAllocated = IndexBelongsTo.GetAllocatedSize() + Arrays.GetAllocatedSize() + Sets.GetAllocatedSize();

	uint32 ArraysMemory = 0;
	for (const TArray<int32>& ArrayIt : Arrays)
	{
		ArraysMemory += ArrayIt.GetAllocatedSize();
	}

	uint32 SetsMemory = 0;
	for (const TSet<int32>& SetsIt : Sets)
	{
		SetsMemory += SetsIt.GetAllocatedSize();
	}

	return BaseMemoryAllocated + ArraysMemory + SetsMemory;
}
