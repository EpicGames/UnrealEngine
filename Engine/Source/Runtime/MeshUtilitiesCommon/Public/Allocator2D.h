// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMemory.h"
#include "MeshUtilitiesCommon.h" // For ELightmapUVVersion

struct FMD5Hash;
struct FRect;
struct Rect;
template <typename FuncType> class TFunctionRef;

#define DEBUG_LAYOUT_STATS 0

class FAllocator2D
{
public:
	enum class EMode
	{
		// In this mode, segments represents free space
		// Used for the layout merging usedsegments
		FreeSegments,
		// In this mode, segments represents used space
		// Used for the rasterization of charts.
		UsedSegments
	};

	struct FRect
	{
		uint32 X;
		uint32 Y;
		uint32 W;
		uint32 H;
	};

	struct FSegment
	{
		uint32 StartPos;
		uint32 Length;

		bool operator<( const FSegment& Other ) const { return StartPos < Other.StartPos; }
	};

	struct FRun
	{
		uint32 LongestSegment;
		TArray< FSegment > Segments;

		// Contains mapping from pixel position to first segment index in search range.
		// Only computed when we're in FreeSegments mode to help TestOneRun find
		// the proper segment in O(c) at the expense of a likely cache miss.
		// We'll use a threshold to use this method when the number of iterations
		// saved is worth the cache miss.
		// Obviously, using a uint16 here will reduce cache misses but impose
		// an hopefully enough limitation of texture size 65536x65536 (4GB).
		TArray< uint16 > FreeSegmentsLookup;
	};

public:
	MESHUTILITIESCOMMON_API FAllocator2D( EMode Mode, uint32 Width, uint32 Height, ELightmapUVVersion LayoutVersion );

	// Must clear before using
	MESHUTILITIESCOMMON_API void       Clear();

	MESHUTILITIESCOMMON_API bool       Find( FRect& Rect );
	bool       Test( FRect Rect );
	MESHUTILITIESCOMMON_API void       Alloc( FRect Rect );

	MESHUTILITIESCOMMON_API bool       FindBitByBit( FRect& Rect, const FAllocator2D& Other );
	MESHUTILITIESCOMMON_API bool       FindWithSegments( FRect& Rect, const FAllocator2D& Other, TFunctionRef<bool (const FAllocator2D::FRect&)> IsBestRect ) const;
	bool       Test( FRect Rect, const FAllocator2D& Other );
	MESHUTILITIESCOMMON_API void       Alloc( FRect Rect, const FAllocator2D& Other );

	uint64     GetBit( uint32 x, uint32 y ) const;
	void       SetBit( uint32 x, uint32 y );
	void       ClearBit( uint32 x, uint32 y );

	MESHUTILITIESCOMMON_API void       CreateUsedSegments();
	MESHUTILITIESCOMMON_API void       MergeRun( FRun& Run, const FRun& OtherRun, uint32 RectOffset, uint32 RectLength, uint32 PrimaryResolution /* Resolution along the axis the run belongs to */, uint32 PerpendicularResolution );
	MESHUTILITIESCOMMON_API void       MergeSegments( const FRect& Rect, const FAllocator2D& Other );

	MESHUTILITIESCOMMON_API void       FlipX( const FRect& Rect );
	MESHUTILITIESCOMMON_API void       FlipY( const FRect& Rect );

	MESHUTILITIESCOMMON_API uint32     GetUsedTexels() const;

	MESHUTILITIESCOMMON_API void       CopyRuns( TArray<FRun>& Runs, const TArray<FRun>& OtherRuns, int32 MaxSize );

	// Take control of the copy assignment to reduce the amount of data movement to the strict minimum
	FAllocator2D(const FAllocator2D& Other) = default;
	MESHUTILITIESCOMMON_API FAllocator2D& operator = (const FAllocator2D& Other);

	// Allow to visualize the content in ascii for debugging purpose. (i.e Watch or Immediate window).
	MESHUTILITIESCOMMON_API FString    ToString() const;

	// Get the MD5 hash of the rasterized content
	MESHUTILITIESCOMMON_API FMD5Hash   GetRasterMD5() const;

	uint32     GetRasterWidth()  const { return RasterWidth; }
	uint32     GetRasterHeight() const { return RasterHeight; }

	MESHUTILITIESCOMMON_API void       ResetStats();
	MESHUTILITIESCOMMON_API void       PublishStats( int32 ChartIndex, int32 Orientation, bool bFound, const FRect& Rect, const FRect& BestRect, const FMD5Hash& ChartMD5, TFunctionRef<bool (const FAllocator2D::FRect&)> IsBestRect );

protected:
	MESHUTILITIESCOMMON_API bool       TestOneRun( const FRun& Run, const FRun& OtherRun, uint32 RectOffset, uint32 RectLength, uint32 PrimaryResolution, uint32& OutFailedLength ) const;
	MESHUTILITIESCOMMON_API bool       TestAllRows( const FRect& Rect, const FAllocator2D& Other, uint32& FailedLength ) const;
	MESHUTILITIESCOMMON_API bool       TestAllColumns( const FRect& Rect, const FAllocator2D& Other, uint32& FailedLength ) const;
	MESHUTILITIESCOMMON_API void       InitRuns( TArray<FRun>& Runs, uint32 PrimaryResolution, uint32 PerpendicularRasterSize);
	MESHUTILITIESCOMMON_API void       InitSegments();
	MESHUTILITIESCOMMON_API void       AddUsedSegment( FRun& Run, uint32 StartPos, uint32 Length );

	// Enforce that those cannot be changed in flight
	const EMode              Mode;
	const uint32             Width;
	const uint32             Height;
	const uint32             Pitch;
	const ELightmapUVVersion LayoutVersion;

	uint32                   RasterWidth;
	uint32                   RasterHeight;
	TArray< FRun >           Rows;        // Represent rows in the grid
	TArray< FRun >           Columns;     // Represent columns in the grid (used when version >= Segments2D).
	TArray< uint64 >         Bits;

	// Index inside rows that will be sorted by rows with longest used segment first
	TArray< uint16 >         SortedRowsIndex;
	// Index inside columns that will be sorted by columns with longest used segment first
	TArray< uint16 >         SortedColumnsIndex;

private:
	// Store iteration stats of the principal algorithms.
	struct FStats
	{
		struct FStat
		{
#if DEBUG_LAYOUT_STATS
			inline void operator++(int) { Value++; }
			inline void operator+=(uint64 InValue) { Value += InValue; }
			inline uint64 GetValue() const { return Value; }
		private:
			uint64 Value = 0;
#else
			// These will be optimized away when stats are not activated
			inline void operator++(int) { }
			inline void operator+=(uint64 InValue) { }
			inline uint64 GetValue() const { return 0; }
#endif
		};

		FStat FindWithSegmentsIterationsY;
		FStat FindWithSegmentsIterationsX;
		FStat FindWithSegmentsMovedPastPreviousBest;
		FStat TestAllRowsIterationsY;
		FStat FreeSegmentLookupCount;
		FStat FreeSegmentRangeIterations;
		FStat FreeSegmentFutureIterations;
		FStat FreeSegmentFutureHit;
		FStat FreeSegmentFutureHitStep;
		FStat FreeSegmentFutureMiss;
		FStat FreeSegmentFutureMissStep;

		void Reset()
		{
			FPlatformMemory::Memzero(this, sizeof(FStats));
		}
	};

	mutable FStats Stats;
};

// Returns non-zero if set
inline uint64 FAllocator2D::GetBit( uint32 x, uint32 y ) const
{
	return Bits[ (x >> 6) + y * Pitch ] & ( 1ull << ( x & 63 ) );
}

inline void FAllocator2D::SetBit( uint32 x, uint32 y )
{
	Bits[ (x >> 6) + y * Pitch ] |= ( 1ull << ( x & 63 ) );

	// Keep track of the rasterized dimension to optimize operations on that area only
	if (y >= RasterHeight)
	{
		RasterHeight = y + 1;
	}

	if (x >= RasterWidth)
	{
		RasterWidth = x + 1;
	}
}

inline void FAllocator2D::ClearBit( uint32 x, uint32 y )
{
	Bits[ (x >> 6) + y * Pitch ] &= ~( 1ull << ( x & 63 ) );
}

inline bool FAllocator2D::Test( FRect Rect )
{
	for( uint32 y = Rect.Y; y < Rect.Y + Rect.H; y++ )
	{
		for( uint32 x = Rect.X; x < Rect.X + Rect.W; x++ )
		{
			if( GetBit( x, y ) )
			{
				return false;
			}
		}
	}
	
	return true;
}

inline bool FAllocator2D::Test( FRect Rect, const FAllocator2D& Other )
{
	const uint32 LowShift = Rect.X & 63;
	const uint32 HighShift = 64 - LowShift;

	for( uint32 y = 0; y < Rect.H; y++ )
	{
#if 1
		uint32 ThisIndex = (Rect.X >> 6) + (y + Rect.Y) * Pitch;
		uint32 OtherIndex = y * Pitch;

		// Test a uint64 at a time
		for( uint32 x = 0; x < Rect.W; x += 64 )
		{
			// no need to zero out HighInt on wrap around because Other will always be zero outside Rect.
			uint64 LowInt  = Bits[ ThisIndex ];
			uint64 HighInt = Bits[ ThisIndex + 1 ];

			uint64 ThisInt = (HighInt << HighShift) | (LowInt >> LowShift);
			uint64 OtherInt = Other.Bits[ OtherIndex ];

			if( ThisInt & OtherInt )
			{
				return false;
			}

			ThisIndex++;
			OtherIndex++;
		}
#else
		for( uint32 x = 0; x < Rect.W; x++ )
		{
			if( Other.GetBit( x, y ) && GetBit( x + Rect.X, y + Rect.Y ) )
			{
				return false;
			}
		}
#endif
	}
	
	return true;
}
