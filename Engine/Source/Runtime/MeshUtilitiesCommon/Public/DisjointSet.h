// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FDisjointSet
{
public:
			FDisjointSet() {}
			FDisjointSet( const uint32 Size );
	
	void	Init( uint32 Size );
	void	Reset();
	void	AddDefaulted( uint32 Num = 1 );

	void	Union( uint32 x, uint32 y );
	void	UnionSequential( uint32 x, uint32 y );
	uint32	Find( uint32 i );

	uint32	operator[]( uint32 i ) const	{ return Parents[i]; }

private:
	TArray< uint32 >	Parents;
};

inline FDisjointSet::FDisjointSet( const uint32 Size )
{
	Init( Size );
}

inline void FDisjointSet::Init( uint32 Size )
{
	Parents.SetNumUninitialized( Size, EAllowShrinking::No);
	for( uint32 i = 0; i < Size; i++ )
	{
		Parents[i] = i;
	}
}

inline void FDisjointSet::Reset()
{
	Parents.Reset();
}

inline void FDisjointSet::AddDefaulted( uint32 Num )
{
	uint32 Start = Parents.AddUninitialized( Num );

	for( uint32 i = Start; i < Start + Num; i++ )
	{
		Parents[i] = i;
	}
}

// Union with splicing
inline void FDisjointSet::Union( uint32 x, uint32 y )
{
	uint32 px = Parents[x];
	uint32 py = Parents[y];

	while( px != py )
	{
		// Pick larger
		if( px < py )
		{
			Parents[x] = py;
			if( x == px )
			{
				return;
			}
			x = px;
			px = Parents[x];
		}
		else
		{
			Parents[y] = px;
			if( y == py )
			{
				return;
			}
			y = py;
			py = Parents[y];
		}
	}
}

// Optimized version of Union when iterating for( x : 0 to N ) unioning x with lower indexes.
// Neither x nor y can have already been unioned with an index > x.
inline void FDisjointSet::UnionSequential( uint32 x, uint32 y )
{
	checkSlow( x >= y );
	checkSlow( x == Parents[x] );

	uint32 px = x;
	uint32 py = Parents[y];
	while( px != py )
	{
		Parents[y] = px;
		if( y == py )
		{
			return;
		}
		y = py;
		py = Parents[y];
	}
}

// Find with path compression
inline uint32 FDisjointSet::Find( uint32 i )
{
	// Find root
	uint32 Start = i;
	uint32 Root = Parents[i];
	while( Root != i )
	{
		i = Root;
		Root = Parents[i];
	}

	// Point all nodes on path to root
	i = Start;
	uint32 Parent = Parents[i];
	while( Parent != Root )
	{
		Parents[i] = Root;
		i = Parent;
		Parent = Parents[i];
	}

	return Root;
}