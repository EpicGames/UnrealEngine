// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Allocator2D.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "MeshUtilitiesCommon.h"

template <typename T> class TAtomic;

struct FMeshChart
{
	uint32		FirstTri;
	uint32		LastTri;
	
	FVector2f	MinUV;
	FVector2f	MaxUV;
	
	float		UVArea;
	FVector2f	UVScale;
	FVector2f	WorldScale;
	
	float		UVLengthSum;
	float		WorldLengthSum;

	FVector2f	PackingScaleU;
	FVector2f	PackingScaleV;
	FVector2f	PackingBias;

	int32		Join[4];

	int32		Id; // Store a unique id so that we can come back to the initial Charts ordering when necessary
};

struct FOverlappingCorners;

class FLayoutUV
{
public:

	/**
	 * Abstract triangle mesh view interface that may be used by any module without introducing
	 * a dependency on a concrete mesh type (and thus potentially circular module references).
	 * This abstraction results in a performance penalty due to virtual dispatch,
	 * however it is expected to be insignificant compared to the rest of work done by FLayoutUV
	 * and cache misses due to indexed vertex data access.
	*/
	struct IMeshView
	{
		virtual ~IMeshView() = default;

		virtual uint32      GetNumIndices() const = 0;
		virtual FVector3f   GetPosition(uint32 Index) const = 0;
		virtual FVector3f   GetNormal(uint32 Index) const = 0;
		virtual FVector2f   GetInputTexcoord(uint32 Index) const = 0;

		virtual void        InitOutputTexcoords(uint32 Num) = 0;
		virtual void        SetOutputTexcoord(uint32 Index, const FVector2f& Value) = 0;
	};

	MESHUTILITIESCOMMON_API FLayoutUV( IMeshView& InMeshView );
	void SetVersion( ELightmapUVVersion Version ) { LayoutVersion = Version; }
	MESHUTILITIESCOMMON_API int32 FindCharts( const FOverlappingCorners& OverlappingCorners );
	MESHUTILITIESCOMMON_API bool FindBestPacking( uint32 InTextureResolution );
	MESHUTILITIESCOMMON_API void CommitPackedUVs();

	static MESHUTILITIESCOMMON_API void LogStats();
	static MESHUTILITIESCOMMON_API void ResetStats();
private:
	IMeshView& MeshView;
	ELightmapUVVersion LayoutVersion;

	TArray< FVector2f > MeshTexCoords;
	TArray< uint32 > MeshSortedTris;
	TArray< FMeshChart > MeshCharts;
	uint32 PackedTextureResolution;

	struct FChartFinder;
	struct FChartPacker;

	static MESHUTILITIESCOMMON_API TAtomic<uint64> FindBestPackingCount;
	static MESHUTILITIESCOMMON_API TAtomic<uint64> FindBestPackingCycles;
	static MESHUTILITIESCOMMON_API TAtomic<uint64> FindBestPackingEfficiency;
};
