// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class ELightmapUVVersion : int32
{
	BitByBit = 0,
	Segments = 1,
	SmallChartPacking = 2,
	ScaleChartsOrderingFix = 3,
	ChartJoiningLFix = 4,
	Allocator2DFlipFix = 5,
	ConsiderLightmapPadding = 6,
	ForceLightmapPadding = 7,
	Segments2D = 8,
	OptimalSurfaceArea = 9,
	ScaleByEdgesLength = 10,
	Latest = ScaleByEdgesLength
};

/**
*	Contains the vertices that are most dominated by that bone. Vertices are in Bone space.
*	Not used at runtime, but useful for fitting physics assets etc.
*/
struct FBoneVertInfo
{
	// Invariant: Arrays should be same length!
	TArray<FVector3f>	Positions;
	TArray<FVector3f>	Normals;
};

/** Helper struct for building acceleration structures. */
struct FIndexAndZ
{
	float Z;
	int32 Index;

	/** Default constructor. */
	FIndexAndZ() {}

	/** Initialization constructor. */
	FIndexAndZ(int32 InIndex, FVector3f V)
	{
		Z = 0.30f * V.X + 0.33f * V.Y + 0.37f * V.Z;
		Index = InIndex;
	}
};

/** Sorting function for vertex Z/index pairs. */
struct FCompareIndexAndZ
{
	inline bool operator()(FIndexAndZ const& A, FIndexAndZ const& B) const { return A.Z < B.Z; }
};

/**
* Returns true if the specified points are about equal
*/
inline bool PointsEqual(const FVector3f& V1, const FVector3f& V2, float ComparisonThreshold)
{
	if (FMath::Abs(V1.X - V2.X) > ComparisonThreshold
		|| FMath::Abs(V1.Y - V2.Y) > ComparisonThreshold
		|| FMath::Abs(V1.Z - V2.Z) > ComparisonThreshold)
	{
		return false;
	}
	return true;
}

namespace TriangleUtilities
{
	/*
	 * This function compute the area of a triangle, it will return zero if the triangle is degenerated
	 */
	static float ComputeTriangleArea(const FVector3f& PointA, const FVector3f& PointB, const FVector3f& PointC)
	{
		return FVector3f::CrossProduct((PointB - PointA), (PointC - PointA)).Size() / 2.0f;
	}

	/*
	 * This function compute the angle of a triangle corner, it will return zero if the triangle is degenerated
	 */
	static float ComputeTriangleCornerAngle(const FVector3f& PointA, const FVector3f& PointB, const FVector3f& PointC)
	{
		FVector3f E1 = (PointB - PointA);
		FVector3f E2 = (PointC - PointA);
		//Normalize both edges (unit vector) of the triangle so we get a dotProduct result that will be a valid acos input [-1, 1]
		if (!E1.Normalize() || !E2.Normalize())
		{
			//Return a null ratio if the polygon is degenerate
			return 0.0f;
		}
		float DotProduct = FVector3f::DotProduct(E1, E2);
		return FMath::Acos(DotProduct);
	}
}

/**
 *	[Frisvad 2012, "Building an Orthonormal Basis from a 3D Unit Vector Without Normalization"]
 */
inline FMatrix44f GetTangentBasisFrisvad(FVector3f TangentZ)
{
	FVector3f TangentX;
	FVector3f TangentY;

	if (TangentZ.Z < -0.9999999f)
	{
		TangentX = FVector3f(0, -1, 0);
		TangentY = FVector3f(-1, 0, 0);
	}
	else
	{
		float A = 1.0f / (1.0f + TangentZ.Z);
		float B = -TangentZ.X * TangentZ.Y * A;
		TangentX = FVector3f(1.0f - TangentZ.X * TangentZ.X * A, B, -TangentZ.X);
		TangentY = FVector3f(B, 1.0f - TangentZ.Y * TangentZ.Y * A, -TangentZ.Y);
	}

	FMatrix44f LocalBasis;
	LocalBasis.SetIdentity();
	LocalBasis.SetAxis(0, TangentX);
	LocalBasis.SetAxis(1, TangentY);
	LocalBasis.SetAxis(2, TangentZ);
	return LocalBasis;
}
