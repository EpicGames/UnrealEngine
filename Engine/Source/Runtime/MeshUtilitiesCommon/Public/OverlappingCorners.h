// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "LayoutUV.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"

/**
* Container to hold overlapping corners. For a vertex, lists all the overlapping vertices
*/
struct FOverlappingCorners
{
	FOverlappingCorners() {}

	MESHUTILITIESCOMMON_API FOverlappingCorners(const TArray<FVector3f>& InVertices, const TArray<uint32>& InIndices, float ComparisonThreshold);
	MESHUTILITIESCOMMON_API FOverlappingCorners(const FLayoutUV::IMeshView& MeshView, float ComparisonThreshold);

	/* Resets, pre-allocates memory, marks all indices as not overlapping in preperation for calls to Add() */
	MESHUTILITIESCOMMON_API void Init(int32 NumIndices);

	/* Add overlapping indices pair */
	MESHUTILITIESCOMMON_API void Add(int32 Key, int32 Value);

	/* Sorts arrays, converts sets to arrays for sorting and to allow simple iterating code, prevents additional adding */
	MESHUTILITIESCOMMON_API void FinishAdding();

	/* Estimate memory allocated */
	MESHUTILITIESCOMMON_API uint32 GetAllocatedSize(void) const;

	/**
	* @return array of sorted overlapping indices including input 'Key', empty array for indices that have no overlaps.
	*/
	const TArray<int32>& FindIfOverlapping(int32 Key) const
	{
		check(bFinishedAdding);
		int32 ContainerIndex = IndexBelongsTo[Key];
		return (ContainerIndex != INDEX_NONE) ? Arrays[ContainerIndex] : EmptyArray;
	}

private:
	TArray<int32> IndexBelongsTo;
	TArray< TArray<int32> > Arrays;
	TArray< TSet<int32> > Sets;
	TArray<int32> EmptyArray;
	bool bFinishedAdding = false;
};
